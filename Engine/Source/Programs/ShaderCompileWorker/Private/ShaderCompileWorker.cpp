// Copyright Epic Games, Inc. All Rights Reserved.


// ShaderCompileWorker.cpp : Defines the entry point for the console application.
//

#include "RequiredProgramMainCPPInclude.h"
#include "HAL/ExceptionHandling.h"
#include "ShaderCompileWorkerUtil.h"
#include "SocketSubsystem.h"

#if PLATFORM_MAC
#include <dlfcn.h>
#include <mach-o/dyld.h>
#elif PLATFORM_LINUX
#include <link.h>
#endif

#define DEBUG_USING_CONSOLE	0

static double GLastCompileTime = 0.0;
static int32 GNumProcessedJobs = 0;

enum class EXGEMode
{
	None,
	Xml,
	Intercept
};

static EXGEMode GXGEMode = EXGEMode::None;

inline bool IsUsingXGE()
{
	return GXGEMode != EXGEMode::None;
}

static FShaderCompileWorkerDiagnostics GWorkerDiagnostics;

static void OnXGEJobCompleted(const TCHAR* WorkingDirectory)
{
	if (GXGEMode == EXGEMode::Xml)
	{
		// To signal compilation completion, create a zero length file in the working directory.
		// This is only required in Xml mode.
		delete IFileManager::Get().CreateFileWriter(*FString::Printf(TEXT("%s/Success"), WorkingDirectory), FILEWRITE_EvenIfReadOnly);
	}
}

#if PLATFORM_WINDOWS // Currently only implemented for windows
HMODULE GetUbaModule()
{
	static HMODULE UbaDetoursModule = GetModuleHandleW(L"UbaDetours.dll");
	return UbaDetoursModule;
}
#endif

inline bool IsUsingUBA()
{
	static bool UbaLoaded;
	if (UbaLoaded)
	{
		return true;
	}

#if PLATFORM_WINDOWS // Currently only implemented for windows
	UbaLoaded = GetUbaModule() != nullptr;
#elif PLATFORM_MAC
	for (int i = 0, e = _dyld_image_count() && !UbaLoaded; i != e; i++)
	{
		UbaLoaded = strstr(_dyld_get_image_name(i), "UbaDetours.dylib") != nullptr;
	}
#elif PLATFORM_LINUX
	dl_iterate_phdr([](struct dl_phdr_info *info, size_t size, void *data) { UbaLoaded = UbaLoaded || (info->dlpi_name && strstr(info->dlpi_name, "UbaDetours.so") != nullptr); return 0; }, nullptr);
#endif

return UbaLoaded;
}

#if USING_CODE_ANALYSIS
	[[noreturn]] static inline void ExitWithoutCrash(FSCWErrorCode::ECode ErrorCode, const FString& Message);
#endif

static inline void ExitWithoutCrash(FSCWErrorCode::ECode ErrorCode, const FString& Message)
{
	GWorkerDiagnostics.ErrorCode = ErrorCode;
	FCString::Snprintf(GErrorExceptionDescription, sizeof(GErrorExceptionDescription), TEXT("%s"), *Message);
	UE_LOGF(LogShaders, Fatal, "%ls", *Message);
}


static const FString& GetLocalHostName()
{
	static FString HostName = []
	{
		FString Output;
		ISocketSubsystem::Get()->GetHostName(Output);
		return Output;
	}();
	return HostName;
}


class FWorkLoop
{
public:
	// If we have been idle for 20 seconds then exit. Can be overriden from the cmd line with -TimeToLive=N where N is in seconds (and a float value)
	float TimeToLive = 20.0f;
	int32 NumberToProcess = -1;
	bool DisableFileWrite = false;
	bool KeepInput = false;
	TArray<FString> DebugSourceFiles;
	FString DumpDebugInfoPathOverride;

	FWorkLoop(const TCHAR* ParentProcessIdText,const TCHAR* InWorkingDirectory,const TCHAR* InInputFilename,const TCHAR* InOutputFilename)
	:	ParentProcessId(FCString::Atoi(ParentProcessIdText))
	,	WorkingDirectory(InWorkingDirectory)
	,	InputFilename(InInputFilename)
	,	OutputFilename(InOutputFilename)
	,	InputFilePath(FString(InWorkingDirectory) / InInputFilename)
	,	OutputFilePath(FString(InWorkingDirectory) / InOutputFilename)
	{
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(FCommandLine::Get(), Tokens, Switches);
		for (FString& Switch : Switches)
		{
			if (Switch.StartsWith(TEXT("TimeToLive=")))
			{
				TimeToLive = FCString::Atof(Switch.GetCharArray().GetData() + 11);
			}
			else if (Switch.Equals(TEXT("DisableFileWrite")))
			{
				DisableFileWrite = true;
			}
			else if (Switch.Equals(TEXT("KeepInput")))
			{
				KeepInput = true;
			}
			else if (Switch.StartsWith(TEXT("NumJobs=")))
			{
				NumberToProcess = FCString::Atoi(Switch.GetCharArray().GetData() + 8);
			}
			else if (Switch.StartsWith(TEXT("debugsourcefiles=")))
			{
				FString DelimitedSourceFiles = Switch.RightChop(17);
				DelimitedSourceFiles.ParseIntoArray(DebugSourceFiles, TEXT(","));
				for (FString& DebugSourceFile : DebugSourceFiles)
				{
					// debug source files are given as relative paths from working directory, make them absolute here and normalize
					DebugSourceFile = InWorkingDirectory / DebugSourceFile;
				}
			}
			else if (Switch.StartsWith(TEXT("DumpDebugInfoPath=")))
			{
				DumpDebugInfoPathOverride = FString(Switch.RightChop(18));
			}
		}
	}

	void Loop(FString& CrashOutputFile)
	{
		UE_LOGF(LogShaders, Log, "Entering job loop");
		TRACE_CPUPROFILER_EVENT_SCOPE(Loop);

		int32 NumberProcessed = 0;

		while (true)
		{
			TArray<FShaderCompileJob> SingleJobs;
			TArray<FShaderPipelineCompileJob> PipelineJobs;
			TArray<FString> PipelineJobNames;

			// Read & Process Input
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ReadInput);

				TUniquePtr<FArchive> InputFilePtr = TUniquePtr<FArchive>(OpenInputFile());
				if (!InputFilePtr)
				{
					break;
				}

				// Record time since SCW entered the entry point
				const double BatchProcessStartTime = FPlatformTime::Seconds();
				GWorkerDiagnostics.BatchPreparationTime = BatchProcessStartTime - (NumberProcessed > 0 ? GLastCompileTime : GWorkerDiagnostics.EntryPointTimestamp);
				GWorkerDiagnostics.BatchIndex = NumberProcessed;

				UE_LOGF(LogShaders, Log, "Processing shader");

				auto LoadVirtualSourceFile = [this](int32 VirtualSourceIndex) -> TArray<uint8>
				{
					TArray<uint8> Output;
					TStringBuilder<512> VirtualSourcePath;
					VirtualSourcePath << InputFilePath << VirtualSourceIndex;
					FFileHelper::LoadFileToArray(Output, *VirtualSourcePath, FILEREAD_NoFail);
					return Output;
				};

				FShaderCompileWorkerUtil::ProcessInputFromArchive(InputFilePtr.Get(), DumpDebugInfoPathOverride, DebugSourceFiles,
					LoadVirtualSourceFile, GNumProcessedJobs, SingleJobs, PipelineJobs, PipelineJobNames);

				GLastCompileTime = FPlatformTime::Seconds();
				GWorkerDiagnostics.BatchProcessTime = GLastCompileTime - BatchProcessStartTime;
			}

			GWorkerDiagnostics.ErrorCode = FSCWErrorCode::IsSet() ? FSCWErrorCode::Get() : FSCWErrorCode::Success;

			// Prepare for output
			if (DisableFileWrite)
			{
				// write to in-memory bytestream instead for debugging purposes
				TArray<uint8> MemBlock;
				FMemoryWriter MemWriter(MemBlock);
				FShaderCompileWorkerUtil::WriteToOutputArchive(MemWriter, GetLocalHostName(), GWorkerDiagnostics, GNumProcessedJobs,
					SingleJobs, PipelineJobs, PipelineJobNames);
			}
			else
			{
				// Write worker output file
				{
					TUniquePtr<FArchive> OutputFilePtr = TUniquePtr<FArchive>(CreateOutputArchive());
					check(OutputFilePtr);
					FShaderCompileWorkerUtil::WriteToOutputArchive(*OutputFilePtr, GetLocalHostName(), GWorkerDiagnostics, GNumProcessedJobs,
						SingleJobs, PipelineJobs, PipelineJobNames);
				}

				// Change the output file name to requested one, if we're using a temporary output file
				if (IsUsingTempOutput())
				{
					IFileManager::Get().Move(*OutputFilePath, *TempFilePath);
				}
			}

			// Reset error code as it can receive a new value in subsequent jobs
			FSCWErrorCode::Reset();

			if (IsUsingXGE())
			{
				// To signal compilation completion, create a zero length file in the working directory.
				OnXGEJobCompleted(*WorkingDirectory);

				// We only do one pass per process when using XGE.
				break;
			}

			if (IsUsingUBA())
			{
#if PLATFORM_WINDOWS
				using ubachar = TCHAR;
#else
				using ubachar = char;
#endif
				
				using UbaRequestNextProcessFunc2 = bool(uint32 prevExitCode, ubachar* outArguments, uint32 outArgumentsCapacity, uint32 timeOutMs, bool* outShouldExit);
				static UbaRequestNextProcessFunc2* RequestNextProcess = nullptr;
				if (!RequestNextProcess)
				{
#if PLATFORM_WINDOWS
					if (HMODULE UbaDetoursModule = GetUbaModule())
					{
						RequestNextProcess = (UbaRequestNextProcessFunc2*)(void*)GetProcAddress(UbaDetoursModule, "UbaRequestNextProcess2");
					}
#elif PLATFORM_MAC
					if (void* UbaDetoursHandle = dlopen("libUbaDetours.dylib", RTLD_LAZY))
					{
						RequestNextProcess = (UbaRequestNextProcessFunc2*)(void*)dlsym(UbaDetoursHandle, "UbaRequestNextProcess2");
					}
#elif PLATFORM_LINUX
					if (void* UbaDetoursHandle = dlopen("libUbaDetours.so", RTLD_LAZY))
					{
						RequestNextProcess = (UbaRequestNextProcessFunc2*)(void*)dlsym(UbaDetoursHandle, "UbaRequestNextProcess2");
					}
#endif
					if (!RequestNextProcess)
					{
						break;
					}
				}

				// Request new process
				ubachar Temp[1024];
				uint32 TimeoutMs = 0;
				if (TimeToLive != 0)
				{
					double TimeSinceLastCompile = FPlatformTime::Seconds() - GLastCompileTime;
					if (TimeSinceLastCompile < TimeToLive)
					{
						TimeoutMs = uint32((TimeToLive - TimeSinceLastCompile)*1000.0);
					}
				}

				bool ShouldExit = false; // Ignore this, we always exit after timeout
				if (!RequestNextProcess(0, Temp, 1024, TimeoutMs, &ShouldExit))
				{
					break; // No process available, exit loop
				}

				const TCHAR* Arguments;
#if PLATFORM_WINDOWS
				Arguments = Temp;
#else
				auto Temp2 = StringCast<TCHAR>(Temp);
				Arguments = Temp2.Get();
#endif

				// We got a new process, change inputs and outputs and run again
				
				TArray<FString> Tokens;
				TArray<FString> Switches;
				FCommandLine::Parse(Arguments, Tokens, Switches);
				if (Tokens.Num() < 5)
				{
					UE_LOGF(LogShaders, Error, "Did not get enough arguments for reuse: %ls", Arguments);
					break;
				}

				WorkingDirectory = Tokens[0];
				InputFilename = Tokens[3];
				OutputFilename = Tokens[4];

				InputFilePath = WorkingDirectory / InputFilename;
				OutputFilePath = WorkingDirectory / OutputFilename;

				CrashOutputFile = OutputFilePath;
				continue;
			}

			if (TimeToLive == 0)
			{
				UE_LOGF(LogShaders, Log, "TimeToLive set to 0, exiting after single job");
				break;
			}

			NumberProcessed++;
			if (NumberToProcess > 0 && NumberProcessed > NumberToProcess)
			{
				UE_LOGF(LogShaders, Log, "NumJobs limit hit");
				break;
			}

#if ENABLE_LOW_LEVEL_MEM_TRACKER
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UpdateStatsPerFrame);

				FLowLevelMemTracker::Get().UpdateStatsPerFrame();
			}
#endif
		}

		UE_LOGF(LogShaders, Log, "Exiting job loop");
	}

private:
	const int32 ParentProcessId;
	FString WorkingDirectory;
	FString InputFilename;
	FString OutputFilename;

	FString InputFilePath;
	FString OutputFilePath;
	FString TempFilePath;

	/** Opens an input file, trying multiple times if necessary. */
	FArchive* OpenInputFile()
	{
		FArchive* InputFile = nullptr;
		bool bFirstOpenTry = true;
		while(!InputFile && !IsEngineExitRequested())
		{
			// Try to open the input file that we are going to process
			InputFile = IFileManager::Get().CreateFileReader(*InputFilePath,FILEREAD_Silent);

			if(!InputFile && !bFirstOpenTry)
			{
				CheckExitConditions();
				// Give up CPU time while we are waiting
				FPlatformProcess::Sleep(0.01f);
			}
			bFirstOpenTry = false;
		}
		return InputFile;
	}

	bool IsUsingTempOutput()
	{
		return !TempFilePath.IsEmpty();
	}

	const TCHAR* GetOutputArchivePath()
	{
		return IsUsingTempOutput() ? *TempFilePath : *OutputFilePath;
	}

	FArchive* CreateOutputArchive()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateOutputArchive);

		FArchive* OutputFilePtr = nullptr;
		const double StartTime = FPlatformTime::Seconds();
		bool bResult = false;

		// It seems XGE does not support deleting files.
		// Don't delete the input file if we are running under Incredibuild (or if the cmdline args explicitly told us to keep it).
		// In xml mode, we signal completion by creating a zero byte "Success" file after the output file has been fully written.
		// In intercept mode, completion is signaled by this process terminating.
		// For UBA we can't delete the file when running remotely because there might be a crash or disconnect happening before result is sent back and then we can't retry
		if (!IsUsingXGE() && !KeepInput && !IsUsingUBA())
		{
			do 
			{
				// Remove the input file so that it won't get processed more than once
				bResult = IFileManager::Get().Delete(*InputFilePath);
			} 
			while (!bResult && (FPlatformTime::Seconds() - StartTime < 2));

			if (!bResult)
			{
				ExitWithoutCrash(FSCWErrorCode::CantDeleteInputFile, FString::Printf(TEXT("Couldn't delete input file %s, is it readonly?"), *InputFilePath));
			}
		}

		// To make sure that the process waiting for results won't read unfinished output file,
		// we use a temp file name during compilation. This is only necessary when not using UBA,
		// since UBA output files are virtualized.
		if (!IsUsingUBA())
		{
			do
			{
				FGuid Guid;
				FPlatformMisc::CreateGuid(Guid);
				TempFilePath = WorkingDirectory + Guid.ToString() + TEXT(".tmp");
			} while (IFileManager::Get().FileSize(*TempFilePath) != INDEX_NONE);
		}

		const double StartTime2 = FPlatformTime::Seconds();

		do 
		{
			// Create the output file.
			OutputFilePtr = IFileManager::Get().CreateFileWriter(GetOutputArchivePath(), FILEWRITE_EvenIfReadOnly);
		} 
		while (!OutputFilePtr && (FPlatformTime::Seconds() - StartTime2 < 2));
			
		if (!OutputFilePtr)
		{
			ExitWithoutCrash(FSCWErrorCode::CantSaveOutputFile, FString::Printf(TEXT("Couldn't save output file %s"), GetOutputArchivePath()));
		}

		return OutputFilePtr;
	}

	/** Called in the idle loop, checks for conditions under which the helper should exit */
	void CheckExitConditions()
	{
		if (!InputFilename.Contains(TEXT("Only")))
		{
			FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - InputFilename did not contain 'Only', exiting after one job."));
		}

#if PLATFORM_MAC || PLATFORM_LINUX
		if (!FPlatformMisc::IsDebuggerPresent() && ParentProcessId > 0)
		{
			// If the parent process is no longer running, exit
			if (!FPlatformProcess::IsApplicationRunning(ParentProcessId))
			{
				FString FilePath = FString(WorkingDirectory) + InputFilename;
				checkf(IFileManager::Get().FileSize(*FilePath) == INDEX_NONE, TEXT("Exiting due to the parent process no longer running! FilePath=%s"), *FilePath);
				FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - Parent process no longer running, exiting"));
			}
		}

		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - GLastCompileTime > TimeToLive)
		{
			UE_LOGF(LogShaders, Log, "No jobs found for %f seconds, exiting", (float)(CurrentTime - GLastCompileTime));
			FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - No jobs found for seconds, exiting"));
		}
#else
		// Don't do these if the debugger is present
		//@todo - don't do these if Unreal is being debugged either
		if (!IsDebuggerPresent())
		{
			if (ParentProcessId > 0)
			{
				FString FilePath = FString(WorkingDirectory) + InputFilename;

				bool bParentStillRunning = true;
				HANDLE ParentProcessHandle = OpenProcess(SYNCHRONIZE, false, ParentProcessId);
				// If we couldn't open the process then it is no longer running, exit
				if (ParentProcessHandle == nullptr)
				{
					checkf(IFileManager::Get().FileSize(*FilePath) == INDEX_NONE, TEXT("Exiting due to OpenProcess(ParentProcessId) failing! FilePath=%s"), *FilePath);
					FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - Couldn't OpenProcess, Parent process no longer running, exiting"));
				}
				else
				{
					// If we did open the process, that doesn't mean it is still running
					// The process object stays alive as long as there are handles to it
					// We need to check if the process has signaled, which indicates that it has exited
					uint32 WaitResult = WaitForSingleObject(ParentProcessHandle, 0);
					if (WaitResult != WAIT_TIMEOUT)
					{
						checkf(IFileManager::Get().FileSize(*FilePath) == INDEX_NONE, TEXT("Exiting due to WaitForSingleObject(ParentProcessHandle) signaling! FilePath=%s"), *FilePath);
						FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - WaitForSingleObject signaled, Parent process no longer running, exiting"));
					}
					CloseHandle(ParentProcessHandle);
				}
			}

			const double CurrentTime = FPlatformTime::Seconds();
			// If we have been idle for 20 seconds then exit
			if (CurrentTime - GLastCompileTime > TimeToLive)
			{
				UE_LOGF(LogShaders, Log, "No jobs found for %f seconds, exiting", (float)(CurrentTime - GLastCompileTime));
				FPlatformMisc::RequestExit(false, TEXT("ShaderCompileWorker - No jobs found for %f seconds, exiting"));
			}
		}
#endif
	}
};

/** 
 * Main entrypoint, guarded by a try ... except.
 * This expects 4 parameters:
 *		The image path and name
 *		The working directory path, which has to be unique to the instigating process and thread.
 *		The parent process Id
 *		The thread Id corresponding to this worker
 */
static int32 GuardedMain(int32 argc, TCHAR* argv[], FString& CrashOutputFile)
{
	FString ExtraCmdLine = TEXT("-NOPACKAGECACHE -ReduceThreadUsage -cpuprofilertrace -nocrashreports");

	// When executing tasks remotely through XGE, enumerating files requires tcp/ip round-trips with
	// the initiator, which can slow down engine initialization quite drastically.
	// The idea here is to save the Ini and Modules manager state and reuse them on the workers
	// to avoid all those directory enumeration during engine init.
	FString IniBootstrapFilename;
	FString ModulesBootstrapFilename;

	// Register out-of-memory delegate to report error code on exit
	FCoreDelegates::GetOutOfMemoryDelegate().AddLambda(
		[]()
		{
			FSCWErrorCode::Report(FSCWErrorCode::OutOfMemory);
		}
	);

	if (IsUsingXGE())
	{
		// Tie the bootstrap filenames to the xge job id to refresh bootstraps state every time a new build starts
		// This allows the ini/modules and shadercompilerworker binaries to change between builds.
		FGuid XGJobID;
		if (FGuid::Parse(FPlatformMisc::GetEnvironmentVariable(TEXT("xgJobID")), XGJobID))
		{
			FString XGJobIDString = XGJobID.ToString(EGuidFormats::DigitsWithHyphens);
			IniBootstrapFilename = FString::Printf(TEXT("%s/Bootstrap-%s.inis"), argv[1], *XGJobIDString);
			ModulesBootstrapFilename = FString::Printf(TEXT("%s/Bootstrap-%s.modules"), argv[1], *XGJobIDString);

			ExtraCmdLine.Appendf(TEXT(" -IniBootstrap=\"%s\" -ModulesBootstrap=\"%s\""), *IniBootstrapFilename, *ModulesBootstrapFilename);

			// Use Windows API directly because required CreateFile flags are not supported by our current OS abstraction
#if PLATFORM_WINDOWS
			// This is advantageous to have only a single worker do the init work instead of having all workers
			// do a stampede of the initiator's machine all trying to enumerate directories at the same time.
			// I've seen incoming TCP connections going through the roof (350 connections for 150 virtual CPUs)
			// coming from workers doing all the same directory enumerations.
			// This is not strictly required, but will improve performance when successful.
			// Most likely a local worker will win the race and do a fast init.
			FString MutexFilename = FString::Printf(TEXT("%s/Bootstrap-%s.mutex"), argv[1], *XGJobIDString);

			// We need to implement a mutex scheme through a file for it to work with XGE's file virtualization layer.
			// The first process to successfully create this file will have the honor of doing the complete initialization.
			HANDLE MutexHandle =
				CreateFileW(
					*MutexFilename,
					GENERIC_WRITE,
					0,
					nullptr,
					CREATE_NEW,
					FILE_ATTRIBUTE_NORMAL,
					nullptr);

			if (MutexHandle != INVALID_HANDLE_VALUE)
			{
				// We won the race, proceed to initialization.
				CloseHandle(MutexHandle);
			}
			else
			{
				// Wait until the race winner writes the last bootstrap file
				// Due to a bug in XGE, some workers might never see the new file appear, we must proceed after some timeout value.
				for (int32 Index = 0; Index < 10 && !FPaths::FileExists(ModulesBootstrapFilename); ++Index)
				{
					Sleep(100);
				}
			}
#endif
		}
	}

	GEngineLoop.PreInit(argc, argv, *ExtraCmdLine);
#if DEBUG_USING_CONSOLE
	GLogConsole->Show( true );
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(Main);

	auto AtomicSave = 
		[](const FString& Filename, TFunctionRef<void (const FString& TmpFile)> SaveFunction)
		{
			if (!Filename.IsEmpty() && !FPaths::FileExists(Filename))
			{
				// Use a tmp file for atomic publication and avoid reading incomplete state from other workers
				FString TmpFile = FString::Printf(TEXT("%s-%s"), *Filename, *FGuid::NewGuid().ToString());
				SaveFunction(TmpFile);
				const bool bReplace = false;
				const bool bDoNotRetryOrError = true;
				const bool bEvenIfReadOnly = false;
				const bool bAttributes = false;
				IFileManager::Get().Move(*Filename, *TmpFile, bReplace, bEvenIfReadOnly, bAttributes, bDoNotRetryOrError);
				// In case this process lost the race and wasn't able to move the file, discard the tmp file.
				IFileManager::Get().Delete(*TmpFile);
			}
		};

	AtomicSave(IniBootstrapFilename,     [](const FString& TmpFile) { GConfig->SaveCurrentStateForBootstrap(*TmpFile); });
	AtomicSave(ModulesBootstrapFilename, [](const FString& TmpFile) { FModuleManager::Get().SaveCurrentStateForBootstrap(*TmpFile); });

	// Explicitly load ShaderPreprocessor module so it will run its initialization step
	FModuleManager::LoadModuleChecked<IModuleInterface>(TEXT("ShaderPreprocessor"));

	if (FShaderCompileWorkerUtil::GetShaderFormats().IsEmpty())
	{
		ExitWithoutCrash(FSCWErrorCode::NoTargetShaderFormatsFound, TEXT("No target shader formats found!"));
	}

	GLastCompileTime = FPlatformTime::Seconds();

#if PLATFORM_WINDOWS
	//@todo - would be nice to change application name or description to have the ThreadId in it for debugging purposes
	SetConsoleTitle(argv[3]);
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(FWorkLoop);

	FWorkLoop WorkLoop(argv[2], argv[1], argv[4], argv[5]);
	WorkLoop.Loop(CrashOutputFile);
	

	return 0;
}


static int32 GuardedMainWrapper(int32 ArgC, TCHAR* ArgV[], FString& CrashOutputFile)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	// We need to know whether we are using XGE now, in case an exception
	// is thrown before we parse the command line inside GuardedMain.
	if ((ArgC > 6) && FCString::Strcmp(ArgV[6], TEXT("-xge_int")) == 0)
	{
		GXGEMode = EXGEMode::Intercept;
	}
	else if ((ArgC > 6) && FCString::Strcmp(ArgV[6], TEXT("-xge_xml")) == 0)
	{
		GXGEMode = EXGEMode::Xml;
	}
	else
	{
		GXGEMode = EXGEMode::None;
	}

	int32 ReturnCode = 0;
#if PLATFORM_WINDOWS
	if (FPlatformMisc::IsDebuggerPresent())
#endif
	{
		ReturnCode = GuardedMain(ArgC, ArgV, CrashOutputFile);
	}
#if PLATFORM_WINDOWS
	else
	{
		// Don't want 32 dialogs popping up when SCW fails
		GUseCrashReportClient = false;
		FString ExceptionMsg;
		FString ExceptionCallStack;
		__try
		{
			GIsGuarded = 1;
			ReturnCode = GuardedMain(ArgC, ArgV, CrashOutputFile);
			GIsGuarded = 0;
		}
		__except(HandleShaderCompileException(GetExceptionInformation(), ExceptionMsg, ExceptionCallStack))
		{
			// Put app into critical error mode to allow dumping logs from memory to disk
			GIsCriticalError = true;

			if (TUniquePtr<FArchive> OutputFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*CrashOutputFile, FILEWRITE_EvenIfReadOnly)))
			{
				if (GWorkerDiagnostics.ErrorCode == FSCWErrorCode::Success)
				{
					if (FSCWErrorCode::IsSet())
					{
						// Use the value set inside the shader format
						GWorkerDiagnostics.ErrorCode = FSCWErrorCode::Get();
					}
					else
					{
						// Something else failed before we could set the error code, so mark it as a General Crash
						GWorkerDiagnostics.ErrorCode = FSCWErrorCode::GeneralCrash;
					}
				}

				int64 FileSizePosition = FShaderCompileWorkerUtil::WriteOutputFileHeader(*OutputFile, GetLocalHostName(), GWorkerDiagnostics, GNumProcessedJobs, ExceptionCallStack.Len(), *ExceptionCallStack, ExceptionMsg.Len(), *ExceptionMsg);

				int32 NumBatches = 0;
				*OutputFile << NumBatches;
				*OutputFile << NumBatches;

				FShaderCompileWorkerUtil::UpdateFileSize(*OutputFile, FileSizePosition);
			}

			if (IsUsingXGE())
			{
				ReturnCode = 1;
				OnXGEJobCompleted(ArgV[1]);
			}
			else if (GetUbaModule())
			{
				ReturnCode = GWorkerDiagnostics.ErrorCode;
			}
		}
	}
#endif

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return ReturnCode;
}

IMPLEMENT_APPLICATION(ShaderCompileWorker, "ShaderCompileWorker")


/**
 * Application entry point
 *
 * @param	ArgC	Command-line argument count
 * @param	ArgV	Argument strings
 */

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	// Record timestamp when entry point is entered
	GWorkerDiagnostics.EntryPointTimestamp = FPlatformTime::Seconds();

	// Redirect for special XGE utilities...
	extern bool XGEMain(int ArgC, TCHAR* ArgV[], int32& ReturnCode);
	{
		int32 ReturnCode;
		if (XGEMain(ArgC, ArgV, ReturnCode))
		{
			return ReturnCode;
		}
	}

	FString OutputFilePath;

	if (ArgC < 6)
	{
		printf("ShaderCompileWorker (v%d) is called by UnrealEditor, it requires specific command line arguments.\n", ShaderCompileWorkerOutputVersion);
		return -1;
	}

	// Game exe can pass any number of parameters through with appGetSubprocessCommandline
	// so just make sure we have at least the minimum number of parameters.
	check(ArgC >= 6);

	OutputFilePath = ArgV[1];
	OutputFilePath += ArgV[5];

	return GuardedMainWrapper(ArgC, ArgV, OutputFilePath);
}
