// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Async.h"
#include "HAL/PlatformNamedPipe.h"
#include "Serialization/MemoryReader.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryWriter.h"
#include "HAL/ExceptionHandling.h"
#include "LaunchEngineLoop.h"

#if PLATFORM_WINDOWS // XGE is only supported on windows platforms

// Enable this to make the controller wait for debugger attachment on startup.
#define WAIT_FOR_DEBUGGER 0

class FXGEControlWorker
{
	const FString PipeName;
	FProcHandle XGConsoleProcHandle;

	FPlatformNamedPipe InputNamedPipe;
	FPlatformNamedPipe OutputNamedPipe;

	struct FTask
	{
		uint32 ID;
		FString Executable;
		FString Arguments;
		FProcHandle Handle;
	};

	FCriticalSection* CS;

	TSet<FTask*> CurrentTasks;

	bool bShutdown;

	TFuture<void> InputThreadFuture, OutputThreadFuture;

	void InputThreadProc();
	void OutputThreadProc();

public:
	FXGEControlWorker(const FString& PipeName);
	~FXGEControlWorker();

	bool Init();

	void WaitForExit();
};

FXGEControlWorker::FXGEControlWorker(const FString& PipeName)
	: PipeName(PipeName)
	, CS(new FCriticalSection)
	, bShutdown(false)
{}

FXGEControlWorker::~FXGEControlWorker()
{
	if (CS)
	{
		delete CS;
		CS = nullptr;
	}

	if (CurrentTasks.Num() > 0)
	{
		// We are shutting down whilst tasks are still in flight. Terminate and close the handle to the parent XGConsole process.
		// Otherwise there are cases where XGE leaves the build running despite this worker process exiting.
		if (XGConsoleProcHandle.IsValid())
		{
			FPlatformProcess::TerminateProc(XGConsoleProcHandle); // This usually sends the Ctrl+C termination signal to this process, so lines after this point may not execute.
			FPlatformProcess::CloseProc(XGConsoleProcHandle);
			XGConsoleProcHandle.Reset();
		}
	}
}

bool FXGEControlWorker::Init()
{
	// Create the output pipe as a server...
	if (!OutputNamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s-B"), *PipeName), true, false))
		return false;

	// Connect the input pipe (engine is the server)...
	if (!InputNamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s-A"), *PipeName), false, false))
		return false;

	// Connect the output pipe (engine is the client)...
	if (!OutputNamedPipe.OpenConnection())
		return false;

	// Read the process ID of the parent xgConsole process
	uint32 XGConsoleProcID = 0;
	if (!InputNamedPipe.ReadBytes(sizeof(XGConsoleProcID), &XGConsoleProcID))
		return false;

	// Attempt to open the parent process handle
	XGConsoleProcHandle = FPlatformProcess::OpenProcess(XGConsoleProcID);
	if (!XGConsoleProcHandle.IsValid() || !FPlatformProcess::IsProcRunning(XGConsoleProcHandle))
		return false;

	// Connection successful, start the worker threads
	InputThreadFuture = Async(EAsyncExecution::Thread, [this]() { InputThreadProc(); });
	OutputThreadFuture = Async(EAsyncExecution::Thread, [this]() { OutputThreadProc(); });
	return true;
}

void FXGEControlWorker::WaitForExit()
{
	InputThreadFuture.Wait();
	OutputThreadFuture.Wait();
}

void FXGEControlWorker::OutputThreadProc()
{
	TArray<uint8> WriteBuffer;

	while (!bShutdown)
	{
		FPlatformProcess::Sleep(0.1f);
		FScopeLock Lock(CS);

		for (auto TasksIter = CurrentTasks.CreateIterator(); TasksIter; ++TasksIter)
		{
			FTask* Task = *TasksIter;
			if (Task->Handle.IsValid() && !FPlatformProcess::IsProcRunning(Task->Handle))
			{
				// Process has completed. Remove the task from the map.
				TasksIter.RemoveCurrent();

				// Grab the process return code and close the handle
				int32 ReturnCode = 0;
				FPlatformProcess::GetProcReturnCode(Task->Handle, &ReturnCode);
				FPlatformProcess::CloseProc(Task->Handle);

				// Write the completion event to the output pipe
				WriteBuffer.Reset();
				FMemoryWriter Writer(WriteBuffer);
				Writer << Task->ID;
				Writer << ReturnCode;

				delete Task;

				if (!OutputNamedPipe.WriteBytes(WriteBuffer.Num(), WriteBuffer.GetData()))
				{
					// Writing to the pipe failed.
					bShutdown = true;
					break;
				}
			}
		}
	}
}

void FXGEControlWorker::InputThreadProc()
{
	TArray<uint8> ReadBuffer;
	uint32 TotalLength;

	while (!bShutdown && InputNamedPipe.ReadBytes(sizeof(TotalLength), &TotalLength))
	{
		ReadBuffer.Reset(TotalLength);
		ReadBuffer.AddUninitialized(TotalLength);

		if (!InputNamedPipe.ReadBytes(TotalLength, ReadBuffer.GetData()))
			break;

		FMemoryReader Reader(ReadBuffer);

		FTask* Task = new FTask();
		Reader << Task->ID;
		Reader << Task->Executable;
		Reader << Task->Arguments;

		// Launch the process with normal priority.
		Task->Handle = FPlatformProcess::CreateProc(*Task->Executable, *Task->Arguments, true, false, false, nullptr, 0, nullptr, nullptr);

		FScopeLock Lock(CS);
		CurrentTasks.Add(Task);
	}

	bShutdown = true;
}

int32 XGEController_GuardedMain(int32 ArgC, TCHAR* ArgV[])
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	GEngineLoop.PreInit(ArgC, ArgV, TEXT("-NOPACKAGECACHE -Multiprocess"));

	if (ArgC != 3)
	{
		// Invalid command line arguments.
		return 1;
	}

	FXGEControlWorker Instance(ArgV[2]);
	if (!Instance.Init())
	{
		// Failed to initialize connection with engine.
		return 2;
	}

	Instance.WaitForExit();

	return 0;
}

// XGE Controller mode is used for the interception interface. The worker establishes a two-way
// communication with the parent engine via named pipes, and submits jobs that arrive from the
// engine on XGE. Completion notifications are submitted back to the engine through the named pipe.
int32 XGEController_Main(int ArgC, TCHAR* ArgV[])
{
#if WAIT_FOR_DEBUGGER
	while (!FPlatformMisc::IsDebuggerPresent())
		FPlatformProcess::Sleep(1.0f);

	UE_DEBUG_BREAK();
#endif

	int32 ReturnCode = 0;

	if (FPlatformMisc::IsDebuggerPresent())
	{
		ReturnCode = XGEController_GuardedMain(ArgC, ArgV);
	}
	else
	{
		__try
		{
			GIsGuarded = 1;
			ReturnCode = XGEController_GuardedMain(ArgC, ArgV);
			GIsGuarded = 0;
		}
		__except (ReportCrash(GetExceptionInformation()))
		{
			ReturnCode = 999;
		}
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return ReturnCode;
}

#endif // PLATFORM_WINDOWS

// XGE Monitor mode is used for the xml interface. It monitors both the engine and
// build processes, and terminates the build if the engine process exits.
int32 XGEMonitor_Main(int ArgC, TCHAR* ArgV[])
{
	// Open handles to the two processes
	FProcHandle EngineProc = FPlatformProcess::OpenProcess(FCString::Atoi(ArgV[2]));
	FProcHandle BuildProc = FPlatformProcess::OpenProcess(FCString::Atoi(ArgV[3]));

	if (EngineProc.IsValid() && BuildProc.IsValid())
	{
		// Whilst the build is still in progress
		while (FPlatformProcess::IsProcRunning(BuildProc))
		{
			// Check that the engine is still alive.
			if (!FPlatformProcess::IsProcRunning(EngineProc))
			{
				// The engine has shutdown before the build was stopped.
				// Kill off the build process
				FPlatformProcess::TerminateProc(BuildProc);
				break;
			}

			FPlatformProcess::Sleep(0.01f);
		}
	}

	return 0;
}

// Selects which XGE mode to run according to the command line,
// returning true and a return code if an XGE mode was run.
bool XGEMain(int ArgC, TCHAR* ArgV[], int32& ReturnCode)
{
	if (ArgC == 4 && FCString::Strcmp(ArgV[1], TEXT("-xgemonitor")) == 0)
	{
		ReturnCode = XGEMonitor_Main(ArgC, ArgV);
		return true;
	}
#if PLATFORM_WINDOWS
	else if (ArgC == 3 && FCString::Strcmp(ArgV[1], TEXT("-xgecontroller")) == 0)
	{
		ReturnCode = XGEController_Main(ArgC, ArgV);
		return true;
	}
#endif // PLATFORM_WINDOWS
	else
	{
		return false;
	}
}
