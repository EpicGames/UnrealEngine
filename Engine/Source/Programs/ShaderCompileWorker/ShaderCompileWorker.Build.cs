// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ShaderCompileWorker : ModuleRules
{
	public ShaderCompileWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		bTreatAsEngineModule = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Projects",
				"RenderCore",
				"RHI",
				"SandboxFile",
				"TargetPlatform",
				"ApplicationCore",
				"TraceLog",
				"ShaderCompilerCommon",
				"Sockets",
			});

		// SCW has no console window, so tracked IO/activity visualization is unused overhead
		PublicDefinitions.Add("UE_ENABLE_TRACKED_IO=0");

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
			});

		// Include D3D compiler binaries
		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			RuntimeDependencies.Add(Path.Combine(Target.WindowsPlatform.DirectXDllDir, "d3dcompiler_47.dll"));
		}
	}
}

