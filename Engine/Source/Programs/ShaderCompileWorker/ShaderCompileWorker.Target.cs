// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Editor)]
public class ShaderCompileWorkerTarget : TargetRules
{
	public ShaderCompileWorkerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.UniqueIfNeeded;

		LaunchModuleName = "ShaderCompileWorker";

        if (bUseXGEController && (Target.Platform == UnrealTargetPlatform.Win64) && Configuration == UnrealTargetConfiguration.Development)
        {
			foreach (UnrealArch Arch in Target.Architectures.Architectures)
			{
				string ArchName = Arch.bIsX64 ? "" : Arch.ToString();

				// The interception interface in XGE requires that the parent and child processes have different filenames on disk.
				// To avoid building an entire separate worker just for this, we duplicate the ShaderCompileWorker in a post build step.
				string SrcPath  = "$(BinaryDir)\\$(TargetName)" + ArchName + ".exe";
				string DestPath = "$(BinaryDir)\\XGEControlWorker" + ArchName + ".exe";

				PostBuildSteps.Add(string.Format("echo Copying {0} to {1}", SrcPath, DestPath));
				PostBuildSteps.Add(string.Format("copy /Y /B \"{0}\" /B \"{1}\" >nul:", SrcPath, DestPath));

				AdditionalBuildProducts.Add(DestPath);
			}
        }

		// Turn off various third party features we don't need

		// Currently we force Lean and Mean mode
		bBuildDeveloperTools = false;

		// ShaderCompileWorker isn't localized, so doesn't need ICU
		bCompileICU = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bBuildWithEditorOnlyData = true;
		bCompileCEF3 = false;

		// Force all shader formats to be built and included.
		bForceBuildShaderFormats = true;

		// ShaderCompileWorker is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		// Disable logging, as the workers are spawned often and logging will just slow them down
		GlobalDefinitions.Add("ALLOW_LOG_FILE=0");

		// Allow logging everything to memory so we can decide to dump everything to file when SCW crashed.
		// This still requires the commandline argument -LogToMemory, so the shader compiling manager can decide to enable/disable it at runtime.
		bLoggingToMemoryEnabled = true;
		bUseLoggingInShipping = true;

		// Linking against wer.lib/wer.dll causes XGE to bail when the worker is run on a Windows 8 machine, so turn this off.
		GlobalDefinitions.Add("ALLOW_WINDOWS_ERROR_REPORT_LIB=0");

		// Disable external profiling in ShaderCompiler to improve startup time
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");

		// This removes another thread being created even when -nocrashreports is specified
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");

		if (bShaderCompilerWorkerTrace)
        {
			GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
			GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
			bEnableTrace = true;
		}
	}
}
